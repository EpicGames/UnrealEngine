{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "AES network packet handler",
	"Description" : "Provides a packet handler component to do AES encryption and decryption.",
	"Category" : "Misc",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AESHandlerComponent",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList":
			[
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
			]
		}
	],
	"Plugins":
	[
		{
			"Name": "PlatformCrypto",
			"Enabled": true,
			"PlatformAllowList":
			[
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
			]
		}
	]
}