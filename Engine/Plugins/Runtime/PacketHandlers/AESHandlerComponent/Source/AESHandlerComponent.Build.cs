// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AESHandlerComponent : ModuleRules
{
	public AESHandlerComponent(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"PacketHandler",
				"PlatformCrypto",
				"PlatformCryptoContext",
			}
		);
	}
}