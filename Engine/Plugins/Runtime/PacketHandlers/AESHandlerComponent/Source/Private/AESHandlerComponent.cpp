// Copyright Epic Games, Inc. All Rights Reserved.

#include "AESHandlerComponent.h"
#include "IPlatformCrypto.h"
#include "PlatformCryptoTypes.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

IMPLEMENT_MODULE( FAESHandlerComponentModule, AESHandlerComponent )

const int32 FAESHandlerComponent::KeySizeInBytes;
const int32 FAESHandlerComponent::BlockSizeInBytes;


TSharedPtr<HandlerComponent> FAESHandlerComponentModule::CreateComponentInstance(FString& Options)
{
	TSharedPtr<HandlerComponent> ReturnVal = NULL;

	ReturnVal = MakeShared<FAESHandlerComponent>();

	return ReturnVal;
}


FAESHandlerComponent::FAESHandlerComponent()
	: FEncryptionComponent(FName(TEXT("AESHandlerComponent")))
	, bEncryptionEnabled(false)
{
	EncryptionContext = IPlatformCrypto::Get().CreateContext();
}

void FAESHandlerComponent::SetEncryptionData(const FEncryptionData& EncryptionData)
{
	if (EncryptionData.Key.Num() != KeySizeInBytes)
	{
		UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::SetEncryptionKey. NewKey is not %d bytes long, ignoring.", KeySizeInBytes);
		return;
	}

	Key.Reset(KeySizeInBytes);
	Key.Append(EncryptionData.Key.GetData(), EncryptionData.Key.Num());
}

void FAESHandlerComponent::EnableEncryption()
{
	bEncryptionEnabled = true;
}

void FAESHandlerComponent::DisableEncryption()
{
	bEncryptionEnabled = false;
}

bool FAESHandlerComponent::IsEncryptionEnabled() const
{
	return bEncryptionEnabled;
}

void FAESHandlerComponent::Initialize()
{
	SetActive(true);
	SetState(UE::Handler::Component::State::Initialized);
	Initialized();
}

bool FAESHandlerComponent::IsValid() const
{
	return true;
}

void FAESHandlerComponent::Incoming(FBitReader& Packet)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler AES Decrypt"), STAT_PacketHandler_AES_Decrypt, STATGROUP_Net);

	// Handle this packet
	if (IsValid() && Packet.GetNumBytes() > 0)
	{
		// Check first bit to see whether payload is encrypted
		if (Packet.ReadBit() != 0)
		{
			// If the key hasn't been set yet, we can't decrypt, so ignore this packet. We don't set an error in this case because it may just be an out-of-order packet.
			if (Key.Num() == 0)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Incoming: received encrypted packet before key was set, ignoring.");
				Packet.SetData(nullptr, 0);
				return;
			}

			// Copy remaining bits to a TArray so that they are byte-aligned.
			if (Packet.GetBytesLeft() > 0)
			{
				Ciphertext.Reset();
				Ciphertext.AddUninitialized(Packet.GetBytesLeft());
				Ciphertext[Ciphertext.Num() - 1] = 0;
			}
			else
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Incoming: missing ciphertext");
				Packet.SetError();
				return;
			}

			Packet.SerializeBits(Ciphertext.GetData(), Packet.GetBitsLeft());

			UE_LOGF(PacketHandlerLog, VeryVerbose, "AES packet handler received %ld bytes before decryption.", Ciphertext.Num());

			EPlatformCryptoResult DecryptResult = EPlatformCryptoResult::Failure;
			TArray<uint8> Plaintext = EncryptionContext->Decrypt_AES_256_ECB(Ciphertext, Key, DecryptResult);

			if (DecryptResult == EPlatformCryptoResult::Failure)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Incoming: failed to decrypt packet.");
				Packet.SetError();
				return;
			}

			if (Plaintext.Num() == 0)
			{
				Packet.SetData(nullptr, 0);
				return;
			}

			// Look for the termination bit that was written in Outgoing() to determine the exact bit size.
			uint8 LastByte = Plaintext.Last();

			if (LastByte != 0)
			{
				int32 BitSize = (Plaintext.Num() * 8) - 1;

				// Bit streaming, starts at the Least Significant Bit, and ends at the MSB.
				while (!(LastByte & 0x80))
				{
					LastByte *= 2;
					BitSize--;
				}

				UE_LOGF(PacketHandlerLog, VeryVerbose, "  Have %d bits after decryption.", BitSize);

				Packet.SetData(MoveTemp(Plaintext), BitSize);
			}
			else
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Incoming: malformed packet, last byte was 0.");
				Packet.SetError();
			}
		}
	}
}

void FAESHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("PacketHandler AES Encrypt"), STAT_PacketHandler_AES_Encrypt, STATGROUP_Net);

	// Handle this packet
	if (IsValid() && Packet.GetNumBytes() > 0)
	{
		// Allow for encryption enabled bit and termination bit. Allow resizing to account for encryption padding.
		FBitWriter NewPacket(Packet.GetNumBits() + 2, true);
		NewPacket.WriteBit(bEncryptionEnabled ? 1 : 0);

		if (NewPacket.IsError())
		{
			UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Outgoing: failed to write encryption bit.");
			Packet.SetError();
			return;
		}

		if (bEncryptionEnabled)
		{
			UE_LOGF(PacketHandlerLog, VeryVerbose, "AES packet handler sending %lld bits before encryption.", Packet.GetNumBits());

			// Write a termination bit so that the receiving side can calculate the exact number of bits sent.
			// Same technique used in UNetConnection.
			Packet.WriteBit(1);

			if (Packet.IsError())
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Outgoing: failed to write termination bit.");
				return;
			}

			EPlatformCryptoResult EncryptResult = EPlatformCryptoResult::Failure;
			TArray<uint8> OutCiphertext = EncryptionContext->Encrypt_AES_256_ECB(TArrayView<uint8>(Packet.GetData(), Packet.GetNumBytes()), Key, EncryptResult);

			if (EncryptResult == EPlatformCryptoResult::Failure)
			{
				UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Outgoing: failed to encrypt packet.");
				Packet.SetError();
				return;
			}
			else
			{
				NewPacket.Serialize(OutCiphertext.GetData(), OutCiphertext.Num());

				if (NewPacket.IsError())
				{
					UE_LOGF(PacketHandlerLog, Log, "FAESHandlerComponent::Outgoing: failed to write ciphertext to packet.");
					Packet.SetError();
					return;
				}

				UE_LOGF(PacketHandlerLog, VeryVerbose, "  AES packet handler sending %lld bytes after encryption.", NewPacket.GetNumBytes());
			}
		}
		else
		{
			NewPacket.SerializeBits(Packet.GetData(), Packet.GetNumBits());
		}

		Packet = MoveTemp(NewPacket);
	}
}

int32 FAESHandlerComponent::GetReservedPacketBits() const
{
	// Worst case includes the encryption enabled bit, the termination bit, padding up to the next whole byte, and a block of padding.
	return 2 + 7 + (BlockSizeInBytes * 8);
}

void FAESHandlerComponent::CountBytes(FArchive& Ar) const
{
	FEncryptionComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(FEncryptionComponent);
	Ar.CountBytes(SizeOfThis, SizeOfThis);

	/*
	Note, as of now, EncryptionContext is just typedef'd, but none of the base
	types actually allocated memory directly in their classes (although there may be
	global state).
	if (FEncryptionContext const * const LocalContext = EncrpytionContext.Get())
	{
		LocalContext->CountBytes(Ar);
	}
	*/

	Key.CountBytes(Ar);
	Ciphertext.CountBytes(Ar);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
