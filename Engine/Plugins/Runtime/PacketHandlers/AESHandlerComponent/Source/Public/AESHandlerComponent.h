// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PacketHandler.h"
#include "IPlatformCrypto.h"
#include "EncryptionComponent.h"

/*
* AES256 ECB block encryption component.
*/
class UE_DEPRECATED(5.1, "No longer supported, please use FAESGCMHandlerComponent instead.") AESHANDLERCOMPONENT_API FAESHandlerComponent : public FEncryptionComponent
{
public:
	/**
	 * Default constructor that leaves the Key empty, and encryption disabled.
	 * You must set the key before enabling encryption, or before receiving encrypted
	 * packets, or those operations will fail.
	 */
	FAESHandlerComponent();

	// This handler uses AES256, which has 32-byte keys.
	static const int32 KeySizeInBytes = 32;

	// This handler uses AES256, which has 32-byte keys.
	static const int32 BlockSizeInBytes = 16;

	// Replace the key used for encryption with NewKey if NewKey is exactly KeySizeInBytes long.
	virtual void SetEncryptionData(const FEncryptionData& EncryptionData) override;

	// After calling this, future outgoing packets will be encrypted (until a call to DisableEncryption).
	virtual void EnableEncryption() override;

	// After calling this, future outgoing packets will not be encrypted (until a call to DisableEncryption).
	virtual void DisableEncryption() override;

	// Returns true if encryption is currently enabled.
	virtual bool IsEncryptionEnabled() const override;

	// HandlerComponent interface
	virtual void Initialize() override;
	virtual bool IsValid() const override;
	virtual void Incoming(FBitReader& Packet) override;
	virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;
	virtual int32 GetReservedPacketBits() const override;
	virtual void CountBytes(FArchive& Ar) const override;

private:
	TUniquePtr<FEncryptionContext> EncryptionContext;

	TArray<uint8> Key;

	TArray<uint8> Ciphertext;

	bool bEncryptionEnabled;
};


/**
 * The public interface to this module.
 */
class UE_DEPRECATED(5.1, "No longer supported, please use FAESGCMHandlerComponentModule instead.") FAESHandlerComponentModule : public FPacketHandlerComponentModuleInterface
{
public:
	/* Creates an instance of this component */
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};
